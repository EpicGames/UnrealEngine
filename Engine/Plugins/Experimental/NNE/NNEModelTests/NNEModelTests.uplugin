{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "NNEModelTests",
	"Description": "Infrastructure to import and add neural network model tests to automation tests that run on different NNE runtimes.",
	"Category": "ML",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://dev.epicgames.com/community/learning/courses/e7w/unreal-engine-neural-network-engine-nne",
	"MarketplaceURL": "",
	"SupportURL": "https://forums.unrealengine.com/t/course-neural-network-engine-nne/1162628",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "NNEModelTests",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NNEModelTestsEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}