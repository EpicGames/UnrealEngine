// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NNEModelTests : ModuleRules
{
	public NNEModelTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
                "Core",
                "CoreUObject"
            }
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
                "AssetRegistry",
                "Engine",
                "Json",
                "NNE",
                "RenderCore",
                "RHI"
            }
		);

        PublicDefinitions.Add("NNEMODELTESTS_TENSOR_DATA_ALIGNMENT=64");
        PrivateDefinitions.Add("NNEMODELTESTS_RDG_BUFFER_MULTIPLE=4");
    }
}
