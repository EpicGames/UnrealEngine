// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEModelTests.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Kismet/GameplayStatics.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "NNE.h"
#include "NNEModelTestsJson.h"
#include "NNERuntimeNPU.h"
#include "Templates/SharedPointer.h"

DEFINE_LOG_CATEGORY(LogNNEModelTests);

namespace UE::NNEModelTests::Private
{
	bool InsertAdditionalFile(const FString& RelativePath, const FString& AbsolutePath, TMap<FString, TArray<FString>>& AdditionalFiles)
	{
		if (!FPaths::FileExists(AbsolutePath))
		{
			UE_LOGF(LogNNEModelTests, Error, "Unable to find the file %ls", *AbsolutePath);
			return false;
		}

		if (AdditionalFiles.Contains(RelativePath))
		{
			bool bFound = false;
			for (int32 i = 0; !bFound && i < AdditionalFiles[RelativePath].Num(); i++)
			{
				bFound |= AdditionalFiles[RelativePath][i].Equals(AbsolutePath);
			}
			if (!bFound)
			{
				AdditionalFiles[RelativePath].Add(AbsolutePath);
			}
		}
		else
		{
			AdditionalFiles.Add(RelativePath, { AbsolutePath });
		}
		return true;
	}

	void InsertReferencedAsset(const FString& RelativePath, TArray<FSoftObjectPath>& AssetReferences)
	{
		bool bFound = false;
		for (int32 i = 0; !bFound && i < AssetReferences.Num(); i++)
		{
			bFound |= AssetReferences[i].ToString().Equals(RelativePath);
		}
		if (!bFound)
		{
			AssetReferences.Add(RelativePath);
		}
		else
		{
			bFound = false;
		}
	}

	void GetInputPathAndFileName(const FInput& Input, FString& RelativePath, FString& FileName)
	{
		RelativePath = FPaths::Combine("Buffers", "Inputs");
		FileName = FString("random_") + Input.Type;
		FString Initializer = Input.Initializer;
		if (!Initializer.Equals("random"))
		{
			FileName = FPaths::Combine(FPaths::GetPath(Initializer), FPaths::GetBaseFilename(Initializer));
		}
		FileName += FString(".nned");
	}

	FString GetOutputPath()
	{
		return FPaths::Combine("Buffers", "Outputs");
	}
}

bool UNNEModelTests::InitializeFromFile(const FString& FilePath, TMap<FString, TArray<FString>>& AdditionalFiles, TMap<FString, TSet<FString>>& RuntimeFilters)
{
	using namespace UE::NNEModelTests::Private;

	// Load the file to a string
	if (!FFileHelper::LoadFileToString(ModelTestsDescription, *FilePath))
	{
		return false;
	}

	// Parse the json
	TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(ModelTestsDescription);
	TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
	FModelTests ModelTests;
	if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) || !JsonObject.IsValid() || !ModelTests.FromJson(JsonObject))
	{
		return false;
	}

	// Go through each test
	FString BasePath = FPaths::GetPath(FilePath);
	for (const FModelTest& Test : ModelTests.Tests)
	{
		// Add the model to the additional files that are needed
		FString RelativePath = FPaths::GetPath(Test.Model);
		FString AbsolutePath = FPaths::ConvertRelativePathToFull(BasePath, Test.Model);
		if (!InsertAdditionalFile(RelativePath, AbsolutePath, AdditionalFiles))
		{
			return false;
		}
		InsertReferencedAsset(GetAbsoluteAssetPath(Test.Model), TestAssetReferences);

		// Add all possible runtimes as filter
		if (!RuntimeFilters.Contains(AbsolutePath))
		{
			RuntimeFilters.Add(AbsolutePath, {});
		}
		for (const FEnvironment& Environment : Test.Environments)
		{
			RuntimeFilters[AbsolutePath].Add(Environment.Runtime);
		}

		// Add the inputs
		for (const FInput& Input : Test.Inputs)
		{
			FString FileName;
			GetInputPathAndFileName(Input, RelativePath, FileName);
			AbsolutePath = FPaths::ConvertRelativePathToFull(BasePath, FPaths::Combine(RelativePath, FileName));

			if (!InsertAdditionalFile(RelativePath, AbsolutePath, AdditionalFiles))
			{
				return false;
			}
			InsertReferencedAsset(GetAbsoluteAssetPath(FPaths::Combine(RelativePath, FileName)), TestAssetReferences);
		}

		// Add the outputs
		RelativePath = GetOutputPath();
		FString BaseName = Test.Name.Replace(TEXT("."), TEXT("_")) + FString("_");
		int32 OutputIndex = 0;
		FString RelativeFilePath = FPaths::Combine(RelativePath, BaseName + FString::FromInt(OutputIndex) + FString(".nned"));
		AbsolutePath = FPaths::ConvertRelativePathToFull(BasePath, RelativeFilePath);
		while (FPaths::FileExists(AbsolutePath))
		{
			if (!InsertAdditionalFile(RelativePath, AbsolutePath, AdditionalFiles))
			{
				return false;
			}
			InsertReferencedAsset(GetAbsoluteAssetPath(RelativeFilePath), TestAssetReferences);

			OutputIndex++;
			RelativeFilePath = FPaths::Combine(RelativePath, BaseName + FString::FromInt(OutputIndex) + FString(".nned"));
			AbsolutePath = FPaths::ConvertRelativePathToFull(BasePath, RelativeFilePath);
		}
	}

	return true;
}

bool UNNEModelTests::GetFilteredModelTestParameters(TArray<UE::NNEModelTests::FModelTestParameters>& ModelTestParameters)
{
	using namespace UE::NNEModelTests::Private;

	// Parse the json
	TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(ModelTestsDescription);
	TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
	FModelTests ModelTests;
	if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) || !JsonObject.IsValid() || !ModelTests.FromJson(JsonObject))
	{
		return false;
	}

	// Process each test
	for (const FModelTest& Test : ModelTests.Tests)
	{
		// Collect all runtimes that fit the current environment
		UE::NNEModelTests::FModelTestParameters Parameters;
		for (const FEnvironment& Environment : Test.Environments)
		{
			const FRequirements& Requirements = Environment.Requirements;
			bool bMeetsRequirements = true;

			// Check if the current platform is inside the required platforms (empty PlatformOr accepts all)
			if (Requirements.PlatformOr.Num() > 0)
			{
				bMeetsRequirements = false;
				for (const FString& Platform : Requirements.PlatformOr)
				{
					if (Platform.Equals(UGameplayStatics::GetPlatformName()))
					{
						bMeetsRequirements = true;
					}
				}
			}
			if (!bMeetsRequirements)
			{
				Parameters.Runtimes.Add({ Environment.Interface, Environment.Runtime, Environment.AbsoluteTolerance, Environment.RelativeTolerance, FString::Printf(TEXT("Skipping %s on platform %s as not required"), *Test.Name, *UGameplayStatics::GetPlatformName()) });
				continue;
			}

			// Check the nne shader requirement (empty NNEShaders string and anything other than 'yes' accepts all)
			if (Requirements.NNEShaders.Len() > 0)
			{
				if (Requirements.NNEShaders.Equals("yes"))
				{
					bMeetsRequirements &= FDataDrivenShaderPlatformInfo::GetSupportsNNEShaders(GMaxRHIShaderPlatform);
				}
			}
			if (!bMeetsRequirements)
			{
				Parameters.Runtimes.Add({ Environment.Interface, Environment.Runtime, Environment.AbsoluteTolerance, Environment.RelativeTolerance, FString::Printf(TEXT("Skipping %s due to missing NNE shader support on the current system"), *Test.Name) });
				continue;
			}

			// Check the NPU requirement (empty NPU string and anything other than 'yes' accepts all)
			if (Requirements.NPU.Len() > 0)
			{
				if (Requirements.NPU.Equals("yes"))
				{
					bMeetsRequirements &= UE::NNE::GetAllRuntimeNames<INNERuntimeNPU>().Num() > 0;
				}
			}
			if (!bMeetsRequirements)
			{
				Parameters.Runtimes.Add({ Environment.Interface, Environment.Runtime, Environment.AbsoluteTolerance, Environment.RelativeTolerance, FString::Printf(TEXT("Skipping %s due to missing NPU support on the current system"), *Test.Name) });
				continue;
			}
			
			Parameters.Runtimes.Add({ Environment.Interface, Environment.Runtime, Environment.AbsoluteTolerance, Environment.RelativeTolerance, FString("")});
		}

		if (Parameters.Runtimes.Num() > 0)
		{
			// Fill in the name and the model path
			Parameters.TestName = Test.Name;
			Parameters.ModelPath = GetAbsoluteAssetPath(Test.Model);

			// Add the input information
			for (const FInput& Input : Test.Inputs)
			{
				FString RelativePath;
				FString FileName;
				GetInputPathAndFileName(Input, RelativePath, FileName);
				FString InputPath = GetAbsoluteAssetPath(RelativePath + "/" + FileName);
				Parameters.Inputs.Add({ InputPath, Input.Shape, Input.Type });
			}

			// Add the output buffers
			FString OutputBasePath = (GetBasePath() + GetOutputPath() + "/").Replace(TEXT("//"), TEXT("/"));
			FString OutputBaseName = Test.Name.Replace(TEXT("."), TEXT("_")) + FString("_");
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
			int32 OutputIndex = 0;
			FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(OutputBasePath + OutputBaseName + FString::FromInt(OutputIndex) + "." + OutputBaseName + FString::FromInt(OutputIndex));
			while (AssetData.IsValid())
			{
				Parameters.Outputs.Add(AssetData.GetObjectPathString());
				OutputIndex++;
				AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(OutputBasePath + OutputBaseName + FString::FromInt(OutputIndex) + "." + OutputBaseName + FString::FromInt(OutputIndex));
			}

			// Add the test
			ModelTestParameters.Add(Parameters);
		}
	}

	return true;
}

FString UNNEModelTests::GetBasePath()
{
	TArray<FString> PathParts;
	this->GetPathName().ParseIntoArray(PathParts, TEXT("/"));
	FString BasePath = "/";
	for (int32 i = 0; i < PathParts.Num() - 1; i++)
	{
		BasePath += PathParts[i] + "/";
	}
	return BasePath;
}

FString UNNEModelTests::GetAbsoluteAssetPath(const FString& RelativeAssetPath)
{
	FString BasePath = GetBasePath();

	FString AbsoluteAssetPath = (BasePath + RelativeAssetPath).Replace(TEXT("//"), TEXT("/"));
	AbsoluteAssetPath = FPaths::ChangeExtension(AbsoluteAssetPath, FPaths::GetBaseFilename(RelativeAssetPath));

	return AbsoluteAssetPath;
}

void UNNEModelTestData::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving() || Ar.IsCountingMemory())
	{
		uint64 DataSize = Data.Num();
		Ar << DataSize;
		if (DataSize > 0)
		{
			Ar.Serialize((void*)Data.GetData(), DataSize);
		}
	}
	else if (Ar.IsLoading())
	{
		uint64 DataSize = 0;
		Ar << DataSize;
		if (DataSize > 0)
		{
			Data.SetNumUninitialized(DataSize);
			Ar.Serialize((void*)Data.GetData(), DataSize);
		}
	}
}

bool UNNEModelTestData::InitializeFromFile(const FString& FilePath)
{
	TArray<uint8> UnalignedData;
	bool bResult = FFileHelper::LoadFileToArray(UnalignedData, *FilePath);
	Data = UnalignedData;
	return bResult;
}

TConstArrayView64<uint8> UNNEModelTestData::GetData()
{
	return Data;
}