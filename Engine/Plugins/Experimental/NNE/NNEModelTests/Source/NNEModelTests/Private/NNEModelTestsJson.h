// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Templates/Tuple.h"

namespace UE::NNEModelTests
{
	struct FInput
	{
		FString Path;
		TArray<int32> Shape;
		FString Type;
	};

	struct FRuntimeParameters
	{
		FString Interface;
		FString RuntimeName;
		double AbsoluteError;
		double RelativeError;
		FString SkipReason;
	};

	struct FModelTestParameters
	{
		FString TestName;
		FString ModelPath;
		TArray<FInput> Inputs;
		TArray<FString> Outputs;
		TArray<FRuntimeParameters> Runtimes;
	};

	namespace Private
	{
		struct FRequirements : public FJsonSerializable
		{
			TArray<FString> PlatformOr;
			FString NNEShaders;
			FString NPU;

			BEGIN_JSON_SERIALIZER
				JSON_SERIALIZE_ARRAY("platform_or", PlatformOr);
				JSON_SERIALIZE("nne_shaders", NNEShaders);
				JSON_SERIALIZE("npu", NPU);
			END_JSON_SERIALIZER
		};

		struct FEnvironment : public FJsonSerializable
		{
			FRequirements Requirements;
			FString Interface;
			FString Runtime;
			double RelativeTolerance;
			double AbsoluteTolerance;

			BEGIN_JSON_SERIALIZER
				JSON_SERIALIZE_OBJECT_SERIALIZABLE("requirements", Requirements);
				JSON_SERIALIZE("interface", Interface);
				JSON_SERIALIZE("runtime", Runtime);
				JSON_SERIALIZE("relative-tolerance", RelativeTolerance);
				JSON_SERIALIZE("absolute-tolerance", AbsoluteTolerance);
			END_JSON_SERIALIZER
		};

		struct FInput : public FJsonSerializable
		{
			TArray<int32> Shape;
			FString Type;
			FString Initializer;

			BEGIN_JSON_SERIALIZER
				JSON_SERIALIZE_ARRAY("shape", Shape);
				JSON_SERIALIZE("type", Type);
				JSON_SERIALIZE("initializer", Initializer);
			END_JSON_SERIALIZER
		};

		struct FModelTest : public FJsonSerializable
		{
			FString Name;
			FString Model;
			TArray<FInput> Inputs;
			TArray<FEnvironment> Environments;

			BEGIN_JSON_SERIALIZER
				JSON_SERIALIZE("name", Name);
				JSON_SERIALIZE("model", Model);
				JSON_SERIALIZE_ARRAY_SERIALIZABLE("inputs", Inputs, FInput);
				JSON_SERIALIZE_ARRAY_SERIALIZABLE("environments", Environments, FEnvironment);
			END_JSON_SERIALIZER
		};

		struct FModelTests : public FJsonSerializable
		{
			TArray<FModelTest> Tests;

			BEGIN_JSON_SERIALIZER
				JSON_SERIALIZE_ARRAY_SERIALIZABLE("tests", Tests, FModelTest);
			END_JSON_SERIALIZER
		};
	} // UE::NNEModelTests::Private
} // UE::NNEModelTests