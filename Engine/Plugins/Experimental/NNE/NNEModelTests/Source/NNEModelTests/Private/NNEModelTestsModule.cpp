// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEModelTestsModule.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "HAL/Event.h"
#include "HAL/IConsoleManager.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "NNE.h"
#include "NNEModelData.h"
#include "NNEModelTests.h"
#include "NNEModelTestsJson.h"
#include "NNERuntimeCPU.h"
#include "NNERuntimeNPU.h"
#include "NNERuntimeGPU.h"
#include "NNERuntimeRDG.h"
#include "NNETypes.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RHICommandList.h"
#include "RHIGPUReadback.h"
#include "ShaderParameterMacros.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakInterfacePtr.h"

#include <type_traits>

IMPLEMENT_COMPLEX_AUTOMATION_TEST_PRIVATE(FNNEModelTestBase, FAutomationTestBase, "NNEModelTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter, __FILE__, __LINE__)
void FNNEModelTestBase::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const {}
bool FNNEModelTestBase::RunTest(const FString& Parameters) { return false; }

namespace UE::NNEModelTests::Private
{
	BEGIN_SHADER_PARAMETER_STRUCT(FReadbackPassParameters, )
		RDG_BUFFER_ACCESS_ARRAY(Buffers)
	END_SHADER_PARAMETER_STRUCT()

	static FAutoConsoleCommand NNEModelTests_Reload(
		TEXT("NNEModelTests.Reload"), TEXT("Reload all the NNEModelTests inside this project."),
		FConsoleCommandDelegate::CreateStatic(
			[]()
			{
				FNNEModelTestsModule& ModelTestsModule = FModuleManager::LoadModuleChecked<FNNEModelTestsModule>("NNEModelTests");
				ModelTestsModule.ReloadModelTests();
			}
		)
	);

	static FAutoConsoleCommand NNEModelTests_Run(
		TEXT("NNEModelTests.Run"), TEXT("Run all the tests that are loaded with the NNEModelTests module."),
		FConsoleCommandDelegate::CreateStatic(
			[]()
			{
				FNNEModelTestsModule& ModelTestsModule = FModuleManager::LoadModuleChecked<FNNEModelTestsModule>("NNEModelTests");

				int32 NumSuccesses = 0;
				int32 NumSkipped = 0;
				int32 Total = 0;
				if (ModelTestsModule.RunModelTests(NumSuccesses, NumSkipped, Total))
				{
					UE_LOGF(LogNNEModelTests, Log, "SUCCESS");
					UE_LOGF(LogNNEModelTests, Log, "Summary: %d Tests / %d Fails / %d Skips", Total, Total - NumSuccesses - NumSkipped, NumSkipped);
				}
				else
				{
					UE_LOGF(LogNNEModelTests, Error, "FAIL");
					UE_LOGF(LogNNEModelTests, Error, "Summary: %d Tests / %d Fails / %d Skips", Total, Total - NumSuccesses - NumSkipped, NumSkipped);
				}
			}
		)
	);

	ENNETensorDataType GetTypeFromString(const FString& TypeString)
	{
		if (TypeString.Equals("Char"))
		{
			return ENNETensorDataType::Char;
		}
		if (TypeString.Equals("Boolean"))
		{
			return ENNETensorDataType::Boolean;
		}
		if (TypeString.Equals("Half"))
		{
			return ENNETensorDataType::Half;
		}
		if (TypeString.Equals("Float"))
		{
			return ENNETensorDataType::Float;
		}
		if (TypeString.Equals("Double"))
		{
			return ENNETensorDataType::Double;
		}
		if (TypeString.Equals("Int8"))
		{
			return ENNETensorDataType::Int8;
		}
		if (TypeString.Equals("Int16"))
		{
			return ENNETensorDataType::Int16;
		}
		if (TypeString.Equals("Int32"))
		{
			return ENNETensorDataType::Int32;
		}
		if (TypeString.Equals("Int64"))
		{
			return ENNETensorDataType::Int64;
		}
		if (TypeString.Equals("UInt8"))
		{
			return ENNETensorDataType::UInt8;
		}
		if (TypeString.Equals("UInt16"))
		{
			return ENNETensorDataType::UInt16;
		}
		if (TypeString.Equals("UInt32"))
		{
			return ENNETensorDataType::UInt32;
		}
		if (TypeString.Equals("UInt64"))
		{
			return ENNETensorDataType::UInt64;
		}
		if (TypeString.Equals("Complex64"))
		{
			return ENNETensorDataType::Complex64;
		}
		if (TypeString.Equals("Complex128"))
		{
			return ENNETensorDataType::Complex128;
		}
		if (TypeString.Equals("BFloat16"))
		{
			return ENNETensorDataType::BFloat16;
		}
		return ENNETensorDataType::None;
	}

	template <typename T>
	struct FDefaultElementConverter
	{
		double operator()(const T& Element)
		{
			return (double)Element;
		}
	};

	struct FHalfElementConverter
	{
		double operator()(const uint16& Element)
		{
			return (double)FGenericPlatformMath::LoadHalf(&Element);
		}
	};

	template<typename T, typename F> 
	bool CompareData(void* Result, void* Reference, int32 NumElements, double AbsoluteTolerance, double RelativeTolerance, bool& bContainsNaNs, double& AbsoluteError, double& RelativeError)
	{
		T* ResultBuffer = (T*)Result;
		T* ReferenceBuffer = (T*)Reference;

		F Converter;

		bContainsNaNs = false;
		AbsoluteError = 0.0;
		RelativeError = 0.0;
		bool bResult = true;
		for (int32 i = 0; i < NumElements; i++)
		{
			double ResultElement = Converter(ResultBuffer[i]);
			double ReferenceElement = Converter(ReferenceBuffer[i]);

			if (FMath::IsNaN(ResultElement))
			{
				bContainsNaNs = true;
				return false;
			}

			// See https://numpy.org/doc/stable/reference/generated/numpy.isclose.html
			double AbsDiff = FMath::Abs<double>(ResultElement - ReferenceElement);
			double AbsRef = FMath::Abs<double>(ReferenceElement);
			double Threshold = AbsoluteTolerance + RelativeTolerance * AbsRef;
			if (AbsDiff > Threshold)
			{
				bResult = false;
			}

			double Temp = AbsDiff / AbsRef;
			RelativeError = RelativeError > Temp ? RelativeError : Temp;
			AbsoluteError = AbsoluteError > AbsDiff ? AbsoluteError : AbsDiff;
		}

		return bResult;
	}

	template <typename T>
	struct FDefaultElementStringConverter
	{
		FString operator()(const T& Element)
		{
			if constexpr (std::is_floating_point_v<T>)
			{
				return FString::SanitizeFloat((float)Element);
			}
			else if constexpr (std::is_integral_v<T>)
			{
				return FString::FromInt(Element);
			}
			else
			{
				// static_assert(false, "No specialization found!");
				return {};
			}
		}
	};

	struct FHalfElementStringConverter
	{
		FString operator()(const uint16& Element)
		{
			return FString::SanitizeFloat((float)FGenericPlatformMath::LoadHalf(&Element));
		}
	};

	template<typename T, typename FConverter, bool LogError>
	void DebugPrintBufferContentTyped(const FString& Label, void* Buffer, int32 NumElements, int32 NumShow, int32 ShowOffset)
	{
		const int32 Num = FMath::Max(FMath::Min(ShowOffset + NumShow, NumElements) - ShowOffset, 0);

		T* BufferAccess = (T*)Buffer;
		FConverter Converter;

		TArray<FString> ElementStrings;
		for (int32 i = 0; i < Num; i++)
		{
			ElementStrings.Add(Converter(BufferAccess[ShowOffset + i]));
		}

		if constexpr (LogError)
		{
			UE_LOGF(LogNNEModelTests, Error, "%ls: %ls", *Label, *FString::Join(ElementStrings, TEXT(", ")));
		}
		else
		{
			UE_LOGF(LogNNEModelTests, Log, "%ls: %ls", *Label, *FString::Join(ElementStrings, TEXT(", ")));
		}
	}

	template<bool LogError>
	void DebugPrintBufferContent(const FString& Label, void* Buffer, int32 NumElements, ENNETensorDataType Type, int32 NumShow = 10, int32 ShowOffset = 0)
	{
		switch (Type)
		{
			case ENNETensorDataType::Double:
			UE::NNEModelTests::Private::DebugPrintBufferContentTyped<double, UE::NNEModelTests::Private::FDefaultElementStringConverter<double>, LogError>(Label, Buffer, NumElements, NumShow, ShowOffset);
			break;
		case ENNETensorDataType::Float:
			UE::NNEModelTests::Private::DebugPrintBufferContentTyped<float, UE::NNEModelTests::Private::FDefaultElementStringConverter<float>, LogError>(Label, Buffer, NumElements, NumShow, ShowOffset);
			break;
		case ENNETensorDataType::Half:
			UE::NNEModelTests::Private::DebugPrintBufferContentTyped<uint16, UE::NNEModelTests::Private::FHalfElementStringConverter, LogError>(Label, Buffer, NumElements, NumShow, ShowOffset);
			break;
		case ENNETensorDataType::Int64:
			UE::NNEModelTests::Private::DebugPrintBufferContentTyped<int64, UE::NNEModelTests::Private::FDefaultElementStringConverter<int64>, LogError>(Label, Buffer, NumElements, NumShow, ShowOffset);
			break;
		case ENNETensorDataType::Int32:
			UE::NNEModelTests::Private::DebugPrintBufferContentTyped<int32, UE::NNEModelTests::Private::FDefaultElementStringConverter<int32>, LogError>(Label, Buffer, NumElements, NumShow, ShowOffset);
			break;
		case ENNETensorDataType::Int16:
			UE::NNEModelTests::Private::DebugPrintBufferContentTyped<int16, UE::NNEModelTests::Private::FDefaultElementStringConverter<int16>, LogError>(Label, Buffer, NumElements, NumShow, ShowOffset);
			break;
		case ENNETensorDataType::Int8:
			UE::NNEModelTests::Private::DebugPrintBufferContentTyped<int8, UE::NNEModelTests::Private::FDefaultElementStringConverter<int8>, LogError>(Label, Buffer, NumElements, NumShow, ShowOffset);
			break;
		case ENNETensorDataType::UInt64:
			UE::NNEModelTests::Private::DebugPrintBufferContentTyped<uint64, UE::NNEModelTests::Private::FDefaultElementStringConverter<uint64>, LogError>(Label, Buffer, NumElements, NumShow, ShowOffset);
			break;
		case ENNETensorDataType::UInt32:
			UE::NNEModelTests::Private::DebugPrintBufferContentTyped<uint32, UE::NNEModelTests::Private::FDefaultElementStringConverter<uint32>, LogError>(Label, Buffer, NumElements, NumShow, ShowOffset);
			break;
		case ENNETensorDataType::UInt16:
			UE::NNEModelTests::Private::DebugPrintBufferContentTyped<uint16, UE::NNEModelTests::Private::FDefaultElementStringConverter<uint16>, LogError>(Label, Buffer, NumElements, NumShow, ShowOffset);
			break;
		case ENNETensorDataType::UInt8:
			UE::NNEModelTests::Private::DebugPrintBufferContentTyped<uint8, UE::NNEModelTests::Private::FDefaultElementStringConverter<uint8>, LogError>(Label, Buffer, NumElements, NumShow, ShowOffset);
			break;
		case ENNETensorDataType::Boolean:
			UE::NNEModelTests::Private::DebugPrintBufferContentTyped<bool, UE::NNEModelTests::Private::FDefaultElementStringConverter<bool>, LogError>(Label, Buffer, NumElements, NumShow, ShowOffset);
			break;
		case ENNETensorDataType::Char:
			UE::NNEModelTests::Private::DebugPrintBufferContentTyped<char, UE::NNEModelTests::Private::FDefaultElementStringConverter<char>, LogError>(Label, Buffer, NumElements, NumShow, ShowOffset);
			break;
		default:
			unimplemented();
			break;
		}
	}
}

class FNNEModelTest : public FNNEModelTestBase
{
public:
	FNNEModelTest(const UE::NNEModelTests::FModelTestParameters& InModelTestParameters) : FNNEModelTestBase(InModelTestParameters.TestName), ModelTestParameters(InModelTestParameters) { }

	virtual ~FNNEModelTest() { }

protected:
	virtual FString GetBeautifiedTestName() const override 
	{ 
		return ModelTestParameters.TestName;
	}

	virtual bool CanRunInEnvironment(const FString& TestParams, FString* OutReason, bool* OutWarn) const override
	{
		const int32 RuntimeIndex = FCString::Atoi(*TestParams);
		check(RuntimeIndex >= 0 && RuntimeIndex < ModelTestParameters.Runtimes.Num());

		UE::NNEModelTests::FRuntimeParameters Runtime = ModelTestParameters.Runtimes[RuntimeIndex];
		*OutReason = Runtime.SkipReason;
		*OutWarn = false;
		return Runtime.SkipReason.Len() < 1;
	}

	virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const override
	{
		const int32 NumRuntimes = ModelTestParameters.Runtimes.Num();
		OutBeautifiedNames.Reserve(NumRuntimes);
		OutTestCommands.Reserve(NumRuntimes);

		const FString& FullName = ModelTestParameters.TestName;
		FString ShortName = FullName;
		{
			int32 LastDot = INDEX_NONE;
			if (FullName.FindLastChar(TEXT('.'), LastDot) && (LastDot + 1) < FullName.Len())
			{
				ShortName = FullName.Mid(LastDot + 1);
			}
		}

		for (int32 i = 0; i < NumRuntimes; i++)
		{
			const UE::NNEModelTests::FRuntimeParameters& Runtime = ModelTestParameters.Runtimes[i];
			OutBeautifiedNames.Add((FString::Printf(TEXT("%s::%s:%s"), *ShortName, *Runtime.RuntimeName, *Runtime.Interface)));
			OutTestCommands.Add(FString::FromInt(i));
		}
	}

	virtual bool RunTest(const FString& Parameter) override
	{
		// Load the model
		TObjectPtr<UNNEModelData> ModelData = LoadObject<UNNEModelData>(GetTransientPackage(), *ModelTestParameters.ModelPath);
		if (!ModelData)
		{
			UE_LOGF(LogNNEModelTests, Error, "Unable to load UNNEModelData from %ls", *ModelTestParameters.ModelPath);
			return false;
		}

		// Load the inputs
		TArray<TObjectPtr<UNNEModelTestData>> Inputs;
		TArray<int32> InputOffsets;
		TArray<UE::NNE::FTensorShape> InputShapes;
		TArray<ENNETensorDataType> InputTypes;
		TMap<FString, int32> OffsetMap;
		for (const UE::NNEModelTests::FInput& Input : ModelTestParameters.Inputs)
		{
			TObjectPtr<UNNEModelTestData> InputData = LoadObject<UNNEModelTestData>(GetTransientPackage(), *Input.Path);
			if (!InputData)
			{
				UE_LOGF(LogNNEModelTests, Error, "Unable to load UNNEModelTestData from %ls", *Input.Path);
				return false;
			}
			Inputs.Add(InputData);

			if (OffsetMap.Contains(Input.Path))
			{
				InputOffsets.Add(OffsetMap[Input.Path]);
				OffsetMap[Input.Path] += NNEMODELTESTS_TENSOR_DATA_ALIGNMENT;
			}
			else
			{
				InputOffsets.Add(0);
				OffsetMap.Add(Input.Path, NNEMODELTESTS_TENSOR_DATA_ALIGNMENT);
			}

			TArray<uint32> ShapeArray;
			for (int32 Dim : Input.Shape)
			{
				if (Dim < 0)
				{
					UE_LOGF(LogNNEModelTests, Error, "Invalid shape for input %ls", *Input.Path);
					return false;
				}
				ShapeArray.Emplace(Dim);
			}
			UE::NNE::FTensorShape Shape = UE::NNE::FTensorShape::Make(ShapeArray);
			InputShapes.Add(Shape);

			ENNETensorDataType Type = UE::NNEModelTests::Private::GetTypeFromString(Input.Type);
			if (Type == ENNETensorDataType::None)
			{
				UE_LOGF(LogNNEModelTests, Error, "Invalid type %ls for input %ls", *Input.Type, *Input.Path);
				return false;
			}
			InputTypes.Add(Type);

			if ((uint64)Shape.Volume() * (uint64)UE::NNE::GetTensorDataTypeSizeInBytes(Type) > (uint64)InputData->GetData().Num())
			{
				UE_LOGF(LogNNEModelTests, Error, "Not enough input data for input %ls", *Input.Path);
				return false;
			}
		}
		
		// Load the reference outputs and allocate space for the generated outputs
		TArray<TObjectPtr<UNNEModelTestData>> ReferenceOutputs;
		TArray<TArray<uint8, TAlignedHeapAllocator<NNEMODELTESTS_TENSOR_DATA_ALIGNMENT>>> Outputs;
		for (const FString& OutputPath : ModelTestParameters.Outputs)
		{
			TObjectPtr<UNNEModelTestData> OutputData = LoadObject<UNNEModelTestData>(GetTransientPackage(), *OutputPath);
			if (!OutputData)
			{
				UE_LOGF(LogNNEModelTests, Error, "Unable to load UNNEModelTestData from %ls", *OutputPath);
				return false;
			}
			ReferenceOutputs.Add(OutputData);
			TArray<uint8, TAlignedHeapAllocator<NNEMODELTESTS_TENSOR_DATA_ALIGNMENT>> TempOutput;
			TempOutput.SetNumUninitialized(OutputData->GetData().Num());
			Outputs.Add(MoveTemp(TempOutput));
		}

		const int32 RuntimeIndex = FCString::Atoi(*Parameter);
		check(RuntimeIndex >= 0 && RuntimeIndex < ModelTestParameters.Runtimes.Num());

		UE::NNEModelTests::FRuntimeParameters RuntimeParameters = ModelTestParameters.Runtimes[RuntimeIndex];
		checkf(RuntimeParameters.SkipReason.Len() < 1, TEXT("Trying to run a test that should be skipped (%s)"), *RuntimeParameters.SkipReason);
		
		TSharedPtr<UE::NNE::IModelInstanceRunSync> ModelInstanceRunSync;
		TSharedPtr<UE::NNE::IModelInstanceRDG> ModelInstanceRDG;
		if (RuntimeParameters.Interface.Equals("INNERuntimeCPU"))
		{
			TWeakInterfacePtr<INNERuntimeCPU> Runtime = UE::NNE::GetRuntime<INNERuntimeCPU>(RuntimeParameters.RuntimeName);
			if (!Runtime.IsValid())
			{
				UE_LOGF(LogNNEModelTests, Error, "Could not find runtime %ls implementing interface %ls", *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			TSharedPtr<UE::NNE::IModelCPU> Model = Runtime->CreateModelCPU(ModelData);
			if (!Model.IsValid())
			{
				UE_LOGF(LogNNEModelTests, Error, "Could not create the model from %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath , *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			ModelInstanceRunSync = Model->CreateModelInstanceCPU();
			if (!ModelInstanceRunSync.IsValid())
			{
				UE_LOGF(LogNNEModelTests, Error, "Could not create the model instance from %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}
		}
		else if (RuntimeParameters.Interface.Equals("INNERuntimeGPU"))
		{
			TWeakInterfacePtr<INNERuntimeGPU> Runtime = UE::NNE::GetRuntime<INNERuntimeGPU>(RuntimeParameters.RuntimeName);
			if (!Runtime.IsValid())
			{
				UE_LOGF(LogNNEModelTests, Error, "Could not find runtime %ls implementing interface %ls", *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			TSharedPtr<UE::NNE::IModelGPU> Model = Runtime->CreateModelGPU(ModelData);
			if (!Model.IsValid())
			{
				UE_LOGF(LogNNEModelTests, Error, "Could not create the model from %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			ModelInstanceRunSync = Model->CreateModelInstanceGPU();
			if (!ModelInstanceRunSync.IsValid())
			{
				UE_LOGF(LogNNEModelTests, Error, "Could not create the model instance from %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}
		}
		else if (RuntimeParameters.Interface.Equals("INNERuntimeNPU"))
		{
			TWeakInterfacePtr<INNERuntimeNPU> Runtime = UE::NNE::GetRuntime<INNERuntimeNPU>(RuntimeParameters.RuntimeName);
			if (!Runtime.IsValid())
			{
				UE_LOGF(LogNNEModelTests, Error, "Could not find runtime %ls implementing interface %ls", *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			TSharedPtr<UE::NNE::IModelNPU> Model = Runtime->CreateModelNPU(ModelData);
			if (!Model.IsValid())
			{
				UE_LOGF(LogNNEModelTests, Error, "Could not create the model from %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			ModelInstanceRunSync = Model->CreateModelInstanceNPU();
			if (!ModelInstanceRunSync.IsValid())
			{
				UE_LOGF(LogNNEModelTests, Error, "Could not create the model instance from %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}
		}
		else if (RuntimeParameters.Interface.Equals("INNERuntimeRDG"))
		{
			TWeakInterfacePtr<INNERuntimeRDG> Runtime = UE::NNE::GetRuntime<INNERuntimeRDG>(RuntimeParameters.RuntimeName);
			if (!Runtime.IsValid())
			{
				UE_LOGF(LogNNEModelTests, Error, "Could not find runtime %ls implementing interface %ls", *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			TSharedPtr<UE::NNE::IModelRDG> Model = Runtime->CreateModelRDG(ModelData);
			if (!Model.IsValid())
			{
				UE_LOGF(LogNNEModelTests, Error, "Could not create the model from %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			ModelInstanceRDG = Model->CreateModelInstanceRDG();
			if (!ModelInstanceRDG.IsValid())
			{
				UE_LOGF(LogNNEModelTests, Error, "Could not create the model instance from %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}
		}
		else
		{
			UE_LOGF(LogNNEModelTests, Error, "Unknown interface %ls", *RuntimeParameters.Interface);
			return false;
		}

		// Run the two cases (sync and rdg)
		TArray<UE::NNE::FTensorShape> OutputShapes;
		TArray<ENNETensorDataType> OutputTypes;
		if (ModelInstanceRunSync.IsValid())
		{
			// Set the input tensor shape
			if (ModelInstanceRunSync->SetInputTensorShapes(InputShapes) != UE::NNE::IModelInstanceRunSync::ESetInputTensorShapesStatus::Ok)
			{
				UE_LOGF(LogNNEModelTests, Error, "Failed to set the input shapes of %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			// Create the input bindings
			TArray<UE::NNE::FTensorBindingCPU> InputBindings;
			for (int32 i = 0; i < Inputs.Num(); i++)
			{
				UE::NNE::FTensorBindingCPU Binding;
				Binding.Data = (void*)&Inputs[i]->GetData().GetData()[InputOffsets[i]];
				checkf((uint64)Binding.Data % NNEMODELTESTS_TENSOR_DATA_ALIGNMENT == 0, TEXT("Tensor input data must be aligned"))
				Binding.SizeInBytes = (uint64)InputShapes[i].Volume() * (uint64)UE::NNE::GetTensorDataTypeSizeInBytes(InputTypes[i]);
				InputBindings.Add(Binding);
			}
			if (InputBindings.Num() < 1)
			{
				UE_LOGF(LogNNEModelTests, Error, "Not enough input values to run %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			// Create the output bindings
			TArray<UE::NNE::FTensorBindingCPU> OutputBindings;
			for (int32 i = 0; i < Outputs.Num(); i++)
			{
				UE::NNE::FTensorBindingCPU Binding;
				Binding.Data = (void*)Outputs[i].GetData();
				checkf((uint64)Binding.Data % NNEMODELTESTS_TENSOR_DATA_ALIGNMENT == 0, TEXT("Tensor output data must be aligned"))
				Binding.SizeInBytes = (uint64)Outputs[i].Num();
				OutputBindings.Add(Binding);
			}

			// Run inference
			if (ModelInstanceRunSync->RunSync(InputBindings, OutputBindings) != UE::NNE::IModelInstanceRunSync::ERunSyncStatus::Ok)
			{
				UE_LOGF(LogNNEModelTests, Error, "Failed to run inference on %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			// Get and test the output shapes
			OutputShapes = ModelInstanceRunSync->GetOutputTensorShapes();
			TConstArrayView<UE::NNE::FTensorDesc> OutputDescs = ModelInstanceRunSync->GetOutputTensorDescs();
			if (OutputShapes.Num() != ReferenceOutputs.Num() || OutputShapes.Num() != OutputDescs.Num())
			{
				UE_LOGF(LogNNEModelTests, Error, "Invalid number of outputs when running inference on %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			// Test the output volumes and get the types
			bool bValidOutputs = true;
			for (int32 i = 0; i < OutputDescs.Num(); i++)
			{
				int32 NumOutputBytes = OutputShapes[i].Volume() * OutputDescs[i].GetElementByteSize();
				if (NumOutputBytes != ReferenceOutputs[i]->GetData().Num() || NumOutputBytes != Outputs[i].Num())
				{
					UE_LOGF(LogNNEModelTests, Error, "Invalid number of outputs when running inference on %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
					bValidOutputs = false;
					continue;
				}
				OutputTypes.Add(OutputDescs[i].GetDataType());
			}
			if (!bValidOutputs)
			{
				return false;
			}
		}
		else if (ModelInstanceRDG.IsValid())
		{
			// Set the input tensor shape
			if (ModelInstanceRDG->SetInputTensorShapes(InputShapes) != UE::NNE::IModelInstanceRDG::ESetInputTensorShapesStatus::Ok)
			{
				UE_LOGF(LogNNEModelTests, Error, "Failed to set the input shapes of %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			// Enqueue a render command to launch the rdg test
			FString ModelName = ModelTestParameters.ModelPath;
			FEvent* Signal = FGenericPlatformProcess::GetSynchEventFromPool(true);

#if UE_BUILD_DEBUG
			TMap<FString, TUniquePtr<FString>>& RDGBufferDebugNamesRef = RDGBufferDebugNames;
#endif
			Signal->Reset();

			ENQUEUE_RENDER_COMMAND(NNEModelTests)
			(
				[&ModelName, &Inputs, &InputShapes, &InputTypes, &InputOffsets, &Outputs, ModelInstanceRDG, &Signal
#if UE_BUILD_DEBUG
					,&RDGBufferDebugNamesRef
#endif
				](FRHICommandListImmediate& RHICmdList)
				{
#if UE_BUILD_DEBUG
					// Need to keep dynamic names because RDG does not copy them internally
					auto GetRDGBufferDebugName = [&RDGBufferDebugNamesRef] (const FString& DebugName) -> const TCHAR*
					{
						if (!RDGBufferDebugNamesRef.Contains(DebugName))
						{
							RDGBufferDebugNamesRef.Emplace(DebugName, MakeUnique<FString>(DebugName));
						}
						return **RDGBufferDebugNamesRef[DebugName];
					};
#endif

					// Make sure we run on on the graphics pipeline
					ERHIPipeline Pipeline = RHICmdList.GetPipeline();
					if (Pipeline == ERHIPipeline::None)
					{
						RHICmdList.SwitchPipeline(ERHIPipeline::Graphics);
					}
					FRDGBuilder	RDGBuilder(RHICmdList);

					// Create the input bindings
					TArray<UE::NNE::FTensorBindingRDG> InputBindings;
					for (int32 i = 0; i < Inputs.Num(); i++)
					{
						uint64 NumElements = (uint64)InputShapes[i].Volume();
						uint64 ElementSizeInBytes = (uint64)UE::NNE::GetTensorDataTypeSizeInBytes(InputTypes[i]);
						checkf(NNEMODELTESTS_RDG_BUFFER_MULTIPLE% ElementSizeInBytes == 0 || ElementSizeInBytes % NNEMODELTESTS_RDG_BUFFER_MULTIPLE == 0, TEXT("Invalid element multiple"));
						uint64 MinSizeMultiple = ElementSizeInBytes > NNEMODELTESTS_RDG_BUFFER_MULTIPLE ? ElementSizeInBytes : NNEMODELTESTS_RDG_BUFFER_MULTIPLE;
						uint64 SizeInBytes = FMath::DivideAndRoundUp(NumElements * ElementSizeInBytes, MinSizeMultiple) * MinSizeMultiple;

						UE::NNE::FTensorBindingRDG Binding;
						FRDGBufferDesc Desc = FRDGBufferDesc::CreateBufferDesc(ElementSizeInBytes, FMath::DivideAndRoundUp(SizeInBytes, ElementSizeInBytes));
						Desc.Usage = EBufferUsageFlags(Desc.Usage | BUF_Static);

#if UE_BUILD_DEBUG
						const TCHAR* BufferNameTmpPtr = GetRDGBufferDebugName(ModelName + TEXT(".Input.") + FString::FromInt(i));
						Binding.Buffer = RDGBuilder.CreateBuffer(Desc, BufferNameTmpPtr, ERDGBufferFlags::None);
#else
						Binding.Buffer = RDGBuilder.CreateBuffer(Desc, TEXT("NNEModelTests.InputBuffer"), ERDGBufferFlags::None);
#endif

						RDGBuilder.QueueBufferUpload(Binding.Buffer, &Inputs[i]->GetData().GetData()[InputOffsets[i]], NumElements * ElementSizeInBytes, ERDGInitialDataFlags::NoCopy);
						InputBindings.Add(Binding);
					}

					// Create output bindings
					UE::NNEModelTests::Private::FReadbackPassParameters* ReadbackParameters = RDGBuilder.AllocParameters<UE::NNEModelTests::Private::FReadbackPassParameters>();
					TArray<UE::NNE::FTensorBindingRDG> OutputBindings;
					for (int32 i = 0; i < Outputs.Num(); i++)
					{
						UE::NNE::FTensorBindingRDG Binding;
						FRDGBufferDesc Desc = FRDGBufferDesc::CreateBufferDesc(1, FMath::DivideAndRoundUp(Outputs[i].Num(), NNEMODELTESTS_RDG_BUFFER_MULTIPLE) * NNEMODELTESTS_RDG_BUFFER_MULTIPLE);
						Desc.Usage = EBufferUsageFlags(Desc.Usage | BUF_Static);

#if UE_BUILD_DEBUG
						const TCHAR* BufferNameTmpPtr = GetRDGBufferDebugName(ModelName + TEXT(".Output.") + FString::FromInt(i));
						Binding.Buffer = RDGBuilder.CreateBuffer(Desc, BufferNameTmpPtr, ERDGBufferFlags::None);
#else
						Binding.Buffer = RDGBuilder.CreateBuffer(Desc, TEXT("NNEModelTests.OutputBuffer"), ERDGBufferFlags::None);
#endif

						ReadbackParameters->Buffers.Emplace(Binding.Buffer, ERHIAccess::CopySrc);
						OutputBindings.Add(Binding);
					}

					// Enqueue the neural network
					ModelInstanceRDG->EnqueueRDG(RDGBuilder, InputBindings, OutputBindings);

					// Enqueue the copy back pass, passing the output buffers as arguments to be read
					TArray<FRHIGPUBufferReadback> Readbacks;
					RDGBuilder.AddPass(
						RDG_EVENT_NAME("FReadbackManager_EnqueueReadbacks"),
						ReadbackParameters,
						ERDGPassFlags::Readback | ERDGPassFlags::NeverCull,
						[&ReadbackParameters, &Readbacks, &ModelName, &Outputs](FRHICommandListImmediate& AddPassRHICmdList)
						{
							for (int32 i = 0; i < ReadbackParameters->Buffers.Num(); i++)
							{
								FRHIGPUBufferReadback& Readback = Readbacks.Emplace_GetRef(*(ModelName + FString(".BufferReadback.") + FString::FromInt(i)));
								Readback.EnqueueCopy(AddPassRHICmdList, ReadbackParameters->Buffers[i]->GetRHI(), Outputs[i].Num());
							}
						}
					);

					// Execute the graph and wait for it to be done
					RDGBuilder.Execute();
					RHICmdList.SubmitAndBlockUntilGPUIdle();

					// Copy the results
					for (int32 i = 0; i < Readbacks.Num(); i++)
					{
						FGenericPlatformMemory::Memcpy(Outputs[i].GetData(), Readbacks[i].Lock(Outputs[i].Num()), Outputs[i].Num());
						Readbacks[i].Unlock();
					}

					// Signal that the data is available
					Signal->Trigger();
				}
			);
			Signal->Wait();
			FGenericPlatformProcess::ReturnSynchEventToPool(Signal);

			// Get and test the output shapes
			OutputShapes = ModelInstanceRDG->GetOutputTensorShapes();
			TConstArrayView<UE::NNE::FTensorDesc> OutputDescs = ModelInstanceRDG->GetOutputTensorDescs();
			if (OutputShapes.Num() != ReferenceOutputs.Num() || OutputShapes.Num() != OutputDescs.Num())
			{
				UE_LOGF(LogNNEModelTests, Error, "Invalid number of outputs when running inference on %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				return false;
			}

			// Test the output volumes and get the types
			bool bValidOutputs = true;
			for (int32 i = 0; i < OutputDescs.Num(); i++)
			{
				int32 NumOutputBytes = OutputShapes[i].Volume() * OutputDescs[i].GetElementByteSize();
				if (NumOutputBytes != ReferenceOutputs[i]->GetData().Num() || NumOutputBytes != Outputs[i].Num())
				{
					UE_LOGF(LogNNEModelTests, Error, "Invalid number of outputs when running inference on %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
					bValidOutputs = false;
					continue;
				}
				OutputTypes.Add(OutputDescs[i].GetDataType());
			}
			if (!bValidOutputs)
			{
				return false;
			}
		}
		else
		{
			UE_LOGF(LogNNEModelTests, Error, "Failed to create a model instance from %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
			return false;
		}

		// Evaluate the results
		bool bResult = true;
		for (int32 i = 0; i < Outputs.Num(); i++)
		{
			int32 NumElements = OutputShapes[i].Volume();
			bool bContainsNaNs = false;
			double AbsoluteError = 0.0;
			double RelativeError = 0.0;
			bool bCurrentResult = false;
			bool bUnimplementedDataType = false;
			switch (OutputTypes[i])
			{
				case ENNETensorDataType::Double:
					bCurrentResult = UE::NNEModelTests::Private::CompareData<double, UE::NNEModelTests::Private::FDefaultElementConverter<double>>((void*)Outputs[i].GetData(), (void*)ReferenceOutputs[i]->GetData().GetData(), NumElements, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError, bContainsNaNs, AbsoluteError, RelativeError);
					break;
				case ENNETensorDataType::Float:
					bCurrentResult = UE::NNEModelTests::Private::CompareData<float, UE::NNEModelTests::Private::FDefaultElementConverter<float>>((void*)Outputs[i].GetData(), (void*)ReferenceOutputs[i]->GetData().GetData(), NumElements, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError, bContainsNaNs, AbsoluteError, RelativeError);
					break;
				case ENNETensorDataType::Half:
					bCurrentResult = UE::NNEModelTests::Private::CompareData<uint16, UE::NNEModelTests::Private::FHalfElementConverter>((void*)Outputs[i].GetData(), (void*)ReferenceOutputs[i]->GetData().GetData(), NumElements, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError, bContainsNaNs, AbsoluteError, RelativeError);
					break;
				case ENNETensorDataType::Int64:
					bCurrentResult = UE::NNEModelTests::Private::CompareData<int64, UE::NNEModelTests::Private::FDefaultElementConverter<int64>>((void*)Outputs[i].GetData(), (void*)ReferenceOutputs[i]->GetData().GetData(), NumElements, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError, bContainsNaNs, AbsoluteError, RelativeError);
					break;
				case ENNETensorDataType::Int32:
					bCurrentResult = UE::NNEModelTests::Private::CompareData<int32, UE::NNEModelTests::Private::FDefaultElementConverter<int32>>((void*)Outputs[i].GetData(), (void*)ReferenceOutputs[i]->GetData().GetData(), NumElements, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError, bContainsNaNs, AbsoluteError, RelativeError);
					break;
				case ENNETensorDataType::Int16:
					bCurrentResult = UE::NNEModelTests::Private::CompareData<int16, UE::NNEModelTests::Private::FDefaultElementConverter<int16>>((void*)Outputs[i].GetData(), (void*)ReferenceOutputs[i]->GetData().GetData(), NumElements, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError, bContainsNaNs, AbsoluteError, RelativeError);
					break;
				case ENNETensorDataType::Int8:
					bCurrentResult = UE::NNEModelTests::Private::CompareData<int8, UE::NNEModelTests::Private::FDefaultElementConverter<int8>>((void*)Outputs[i].GetData(), (void*)ReferenceOutputs[i]->GetData().GetData(), NumElements, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError, bContainsNaNs, AbsoluteError, RelativeError);
					break;
				case ENNETensorDataType::UInt64:
					bCurrentResult = UE::NNEModelTests::Private::CompareData<uint64, UE::NNEModelTests::Private::FDefaultElementConverter<uint64>>((void*)Outputs[i].GetData(), (void*)ReferenceOutputs[i]->GetData().GetData(), NumElements, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError, bContainsNaNs, AbsoluteError, RelativeError);
					break;
				case ENNETensorDataType::UInt32:
					bCurrentResult = UE::NNEModelTests::Private::CompareData<uint32, UE::NNEModelTests::Private::FDefaultElementConverter<uint32>>((void*)Outputs[i].GetData(), (void*)ReferenceOutputs[i]->GetData().GetData(), NumElements, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError, bContainsNaNs, AbsoluteError, RelativeError);
					break;
				case ENNETensorDataType::UInt16:
					bCurrentResult = UE::NNEModelTests::Private::CompareData<uint16, UE::NNEModelTests::Private::FDefaultElementConverter<uint16>>((void*)Outputs[i].GetData(), (void*)ReferenceOutputs[i]->GetData().GetData(), NumElements, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError, bContainsNaNs, AbsoluteError, RelativeError);
					break;
				case ENNETensorDataType::UInt8:
					bCurrentResult = UE::NNEModelTests::Private::CompareData<uint8, UE::NNEModelTests::Private::FDefaultElementConverter<uint8>>((void*)Outputs[i].GetData(), (void*)ReferenceOutputs[i]->GetData().GetData(), NumElements, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError, bContainsNaNs, AbsoluteError, RelativeError);
					break;
				case ENNETensorDataType::Boolean:
					bCurrentResult = UE::NNEModelTests::Private::CompareData<bool, UE::NNEModelTests::Private::FDefaultElementConverter<bool>>((void*)Outputs[i].GetData(), (void*)ReferenceOutputs[i]->GetData().GetData(), NumElements, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError, bContainsNaNs, AbsoluteError, RelativeError);
					break;
				case ENNETensorDataType::Char:
					bCurrentResult = UE::NNEModelTests::Private::CompareData<char, UE::NNEModelTests::Private::FDefaultElementConverter<char>>((void*)Outputs[i].GetData(), (void*)ReferenceOutputs[i]->GetData().GetData(), NumElements, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError, bContainsNaNs, AbsoluteError, RelativeError);
					break;
				default:
					bUnimplementedDataType = true;
					break;
			}
			if (bUnimplementedDataType)
			{
				UE_LOGF(LogNNEModelTests, Error, "Unimplemented data type in %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
				bResult = false;
			}
			else if (!bCurrentResult)
			{
				UE_LOGF(LogNNEModelTests, Error, "Wrong results in output %d when running %ls using the runtime %ls implementing the interface %ls:\n\tContains NaNs: %ls\n\tMaximum absolute error: %lf\n\tMaximum relative error: %lf\n\tAbsolute error tolerance: %lf\n\tRelative error tolerance: %lf", i, *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface, bContainsNaNs ? TEXT("Yes") : TEXT("No"), AbsoluteError, RelativeError, RuntimeParameters.AbsoluteError, RuntimeParameters.RelativeError);
				bResult = false;
			}
		}

		if (bResult)
		{
			UE_LOGF(LogNNEModelTests, Display, "Success running %ls using the runtime %ls implementing the interface %ls", *ModelTestParameters.ModelPath, *RuntimeParameters.RuntimeName, *RuntimeParameters.Interface);
		}
		else
		{
			constexpr int32 NumShow = 10;
			constexpr int32 ShowOffset = 0;

			UE_LOGF(LogNNEModelTests, Error, "Debug partial view of input buffers content at position %d:", ShowOffset);

			for (int32 i = 0; i < Inputs.Num(); i++)
			{
				UE::NNEModelTests::Private::DebugPrintBufferContent<true>(FString::Printf(TEXT("Input   [%d]"), i), (void*)&Inputs[i]->GetData().GetData()[InputOffsets[i]], InputShapes[i].Volume(), InputTypes[i], NumShow, ShowOffset);
			}

			UE_LOGF(LogNNEModelTests, Error, "Debug partial view of output buffers content at position %d:", ShowOffset);

			for (int32 i = 0; i < Outputs.Num(); i++)
			{				
				UE::NNEModelTests::Private::DebugPrintBufferContent<true>(FString::Printf(TEXT("Output   [%d]"), i), (void*)Outputs[i].GetData(), OutputShapes[i].Volume(), OutputTypes[i], NumShow, ShowOffset);
				UE::NNEModelTests::Private::DebugPrintBufferContent<true>(FString::Printf(TEXT("Reference[%d]"), i), (void*)ReferenceOutputs[i]->GetData().GetData(), OutputShapes[i].Volume(), OutputTypes[i], NumShow, ShowOffset);
			}
		}

		return bResult;
	}

private:
	UE::NNEModelTests::FModelTestParameters ModelTestParameters;

#if UE_BUILD_DEBUG
	TMap<FString, TUniquePtr<FString>> RDGBufferDebugNames;
#endif

protected:
	friend class FNNEModelTestsModule;
};

void FNNEModelTestsModule::StartupModule()
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	if (AssetRegistry.IsLoadingAssets())
	{
		FilesLoadedHandle = AssetRegistry.OnFilesLoaded().AddRaw(this, &FNNEModelTestsModule::OnAssetRegistryReady);
	}
	else
	{
		ReloadModelTests();
	}
}

void FNNEModelTestsModule::ShutdownModule()
{
	if (FilesLoadedHandle.IsValid() && FModuleManager::Get().IsModuleLoaded("AssetRegistry"))
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

		AssetRegistry.OnFilesLoaded().Remove(FilesLoadedHandle);
		FilesLoadedHandle.Reset();
	}
	ModelTestsMap.Empty();
}

void FNNEModelTestsModule::OnAssetRegistryReady()
{
	if (FilesLoadedHandle.IsValid())
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

		AssetRegistry.OnFilesLoaded().Remove(FilesLoadedHandle);
		FilesLoadedHandle.Reset();
	}
	ReloadModelTests();
}

void FNNEModelTestsModule::ReloadModelTests()
{
	ModelTestsMap.Empty();

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	TArray<FAssetData> AssetDataArray;
	AssetRegistry.GetAssetsByClass(UNNEModelTests::StaticClass()->GetClassPathName(), AssetDataArray);
	
	for (const FAssetData& AssetData : AssetDataArray)
	{
		TObjectPtr<UNNEModelTests> ModelTests = Cast<UNNEModelTests>(AssetData.GetAsset());
		if (!ModelTests)
		{
			continue;
		}
		if (!LoadModelTests(ModelTests))
		{
			UE_LOGF(LogNNEModelTests, Warning, "Failed to load model tests for %ls", *ModelTests->GetPathName());
		}
	}
}

bool FNNEModelTestsModule::RunModelTests(int32& NumSuccesses, int32& NumSkipped, int32& Total)
{
	NumSuccesses = 0;
	NumSkipped = 0;
	Total = 0;

	for (const TPair<FString, TSharedPtr<FNNEModelTest>>& KeyValuePair : ModelTestsMap)
	{
		FNNEModelTest& Test = *KeyValuePair.Value;

		TArray<FString> BeautifiedNames;
		TArray<FString> ParameterNames;
		Test.GetTests(BeautifiedNames, ParameterNames);

		Total += ParameterNames.Num();

		for (int32 i = 0; i < ParameterNames.Num(); i++)
		{
			const FString& BeautifiedName = BeautifiedNames[i];
			const FString& Parameter = ParameterNames[i];

			FString Reason;
			bool bWarn = false;

			if (!Test.CanRunInEnvironment(Parameter, &Reason, &bWarn))
			{
				FString SkippingMessage = FString::Format(TEXT("Test Skipped. Name={{0}} Reason={{1}} Path={{2}}"), { *BeautifiedName, *Reason, TEXT("-") });
				if (bWarn)
				{
					UE_LOGF(LogNNEModelTests, Warning, "%ls", *SkippingMessage);
				}
				else
				{
					UE_LOGF(LogNNEModelTests, Display, "%ls", *SkippingMessage);
				}

				NumSkipped++;

				continue;
			}

			const bool bTestResult = Test.RunTest(Parameter);
			if (bTestResult)
			{
				NumSuccesses++;
			}
		}
	}
	return (NumSuccesses + NumSkipped) == Total;
}

bool FNNEModelTestsModule::LoadModelTests(TObjectPtr<UNNEModelTests> ModelTests)
{
	TArray<UE::NNEModelTests::FModelTestParameters> ModelTestParametersArray;
	if (!ModelTests->GetFilteredModelTestParameters(ModelTestParametersArray))
	{
		return false;
	}
	for (const UE::NNEModelTests::FModelTestParameters& ModelTestParameters : ModelTestParametersArray)
	{
		if (ModelTestsMap.Contains(ModelTestParameters.TestName))
		{
			UE_LOGF(LogNNEModelTests, Warning, "Replacing existing test %ls", *ModelTestParameters.TestName);
			ModelTestsMap.FindAndRemoveChecked(ModelTestParameters.TestName);
		}
		ModelTestsMap.Add(ModelTestParameters.TestName, MakeShared<FNNEModelTest>(ModelTestParameters));
	}
	return true;
}
	
IMPLEMENT_MODULE(FNNEModelTestsModule, NNEModelTests)