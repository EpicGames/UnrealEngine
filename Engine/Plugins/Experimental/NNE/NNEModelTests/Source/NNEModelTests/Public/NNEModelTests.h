// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Logging/LogMacros.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"

#include "NNEModelTests.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogNNEModelTests, Log, All);

namespace UE::NNEModelTests
{
	struct FModelTestParameters;
} // UE::NNEModelTests

UCLASS(BlueprintType, Category = "NNE")
class NNEMODELTESTS_API UNNEModelTests : public UObject
{
	GENERATED_BODY()

public:
	bool InitializeFromFile(const FString& FilePath, TMap<FString, TArray<FString>>& AdditionalFiles, TMap<FString, TSet<FString>>& RuntimeFilters);
	bool GetFilteredModelTestParameters(TArray<UE::NNEModelTests::FModelTestParameters>& ModelTestParameters);

private:
	FString GetBasePath();
	FString GetAbsoluteAssetPath(const FString& RelativeAssetPath);

public:
	FString ReimportPath;

private:
	UPROPERTY()
	FString ModelTestsDescription;

	UPROPERTY()
	TArray<FSoftObjectPath> TestAssetReferences;
};

UCLASS(BlueprintType, Category = "NNE")
class NNEMODELTESTS_API UNNEModelTestData : public UObject
{
	GENERATED_BODY()

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

public:
	bool InitializeFromFile(const FString& FilePath);
	TConstArrayView64<uint8> GetData();

private:
	TArray<uint8, TAlignedHeapAllocator<NNEMODELTESTS_TENSOR_DATA_ALIGNMENT>> Data;
};