// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"

class UNNEModelTests;
class FNNEModelTest;

class FNNEModelTestsModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	NNEMODELTESTS_API void ReloadModelTests();
	NNEMODELTESTS_API bool RunModelTests(int32& NumSuccesses, int32& NumSkipped, int32& Total);

private:
	void OnAssetRegistryReady();
	bool LoadModelTests(TObjectPtr<UNNEModelTests> ModelTests);

	TMap<FString, TSharedPtr<FNNEModelTest>> ModelTestsMap;
	FDelegateHandle FilesLoadedHandle;
};
