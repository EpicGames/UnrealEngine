// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NNEModelTestsEditor : ModuleRules
{
	public NNEModelTestsEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "UnrealEd"
            }
        );

        PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
                "AssetTools",
                "Engine",
				"NNE",
                "NNEModelTests"
            }
		);
	}
}
