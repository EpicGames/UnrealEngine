// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEModelTestsFactory.h"

#include "AssetImportTask.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AutomatedAssetImportData.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "NNEModelData.h"
#include "NNEModelTests.h"
#include "NNEModelTestsModule.h"
#include "Subsystems/ImportSubsystem.h"
#include "UObject/UObjectGlobals.h"

DEFINE_LOG_CATEGORY(LogNNEModelTestsFactory);

UNNEModelTestsFactory::UNNEModelTestsFactory(const FObjectInitializer& ObjectInitializer) : UFactory(ObjectInitializer)
{
	bCreateNew = false;
	bEditorImport = true;
	SupportedClass = UNNEModelTests::StaticClass();
	ImportPriority = DefaultImportPriority;
	Formats.Add("nnet;NNE Model Tests File");
}

UObject* UNNEModelTestsFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, TEXT("NNET"));

	TObjectPtr<UNNEModelTests> ModelTests = NewObject<UNNEModelTests>(InParent, InClass, InName, Flags);
	TMap<FString, TArray<FString>> AdditionalFiles;
	TMap<FString, TSet<FString>> RuntimeFilters;
	if (!ModelTests->InitializeFromFile(Filename, AdditionalFiles, RuntimeFilters))
	{
		return nullptr;
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, ModelTests);

	auto MakeImportTask = [] (const FString& SourceFile, const FString& DestinationPath) -> UAssetImportTask*
		{
			UAssetImportTask* Task = NewObject<UAssetImportTask>();
			Task->Filename = SourceFile;
			Task->DestinationPath = DestinationPath;
			Task->bAutomated = true;
			Task->bReplaceExisting = false;
			return Task;
		};

	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
	IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	for (const TPair<FString, TArray<FString>>& KeyValue : AdditionalFiles)
	{
		const FString DestinationPath = FPackageName::GetLongPackagePath(InParent->GetOutermost()->GetPathName()) + TEXT("/") + KeyValue.Key + TEXT("/");

		TArray<UAssetImportTask*> ImportTasks;
		for (int32 i = 0; i < KeyValue.Value.Num(); i++)
		{
			ImportTasks.Add(MakeImportTask(KeyValue.Value[i], DestinationPath));
		}

		AssetTools.ImportAssetTasks(ImportTasks);

		for (UAssetImportTask* Task : ImportTasks)
		{
			// We only would like to modify UNNEModelData which have non-empty runtime filters (empty runtime filter = "all runtimes")
			if (!RuntimeFilters.Contains(Task->Filename) || RuntimeFilters[Task->Filename].Num() == 0)
			{
				continue;
			}

			UObject* ObjectPtr = nullptr;

			if (const auto& ImportedObjects = Task->GetObjects(); ImportedObjects.Num() > 0)
			{
				ObjectPtr = ImportedObjects[0];
			}
			else
			{
				FString BaseName = FPaths::GetBaseFilename(*Task->Filename);
				AssetTools.SanitizeName(BaseName);

				const FString PackageName = DestinationPath + BaseName;
				const FString ObjectName = PackageName + TEXT(".") + BaseName;

				if (FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(ObjectName); AssetData.IsValid())
				{
					UE_LOGF(LogNNEModelTestsFactory, Log, "Found already imported %ls", *Task->Filename);

					ObjectPtr = AssetData.GetAsset();
				}
				else
				{
					UE_LOGF(LogNNEModelTestsFactory, Error, "Failed to import %ls", *Task->Filename);
					return nullptr;
				}
			}

			if (ObjectPtr)
			{
				UNNEModelData* ModelData = Cast<UNNEModelData>(ObjectPtr);
				if (!ModelData)
				{
					UE_LOGF(LogNNEModelTestsFactory, Error, "Expected %ls to be UNNEModelData", *Task->Filename);
					return nullptr;
				}

				TSet<FString> TargetRuntimes = RuntimeFilters[Task->Filename];

				// If asset has been imported already, runtime filters will be merged
				TargetRuntimes.Append(ModelData->GetTargetRuntimes());

				ModelData->Modify();
				ModelData->SetTargetRuntimes(TargetRuntimes.Array());

				ModelData->PostEditChange();
				ModelData->MarkPackageDirty();
			}
			else
			{
				UE_LOGF(LogNNEModelTestsFactory, Error, "Failed to load %ls", *Task->Filename);
				return nullptr;
			}
		}
	}

	FNNEModelTestsModule& ModelTestsModule = FModuleManager::LoadModuleChecked<FNNEModelTestsModule>("NNEModelTests");
	ModelTestsModule.ReloadModelTests();

	return ModelTests;
}

bool UNNEModelTestsFactory::FactoryCanImport(const FString & Filename)
{
	return Filename.EndsWith(FString(".nnet"));
}

bool UNNEModelTestsFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	TObjectPtr<UNNEModelTests> ModelTests = Cast<UNNEModelTests>(Obj);
	if (ModelTests)
	{
		return true;
	}
	return false;
}

void UNNEModelTestsFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	if (NewReimportPaths.Num() > 0)
	{
		TObjectPtr<UNNEModelTests> ModelTests = Cast<UNNEModelTests>(Obj);
		if (ModelTests)
		{
			ModelTests->ReimportPath = NewReimportPaths[0];
		}
	}
}

EReimportResult::Type UNNEModelTestsFactory::Reimport(UObject* Obj)
{
	if (!Obj || !Obj->IsA(UNNEModelTests::StaticClass()))
	{
		return EReimportResult::Failed;
	}

	TObjectPtr<UNNEModelTests> ModelTests = Cast<UNNEModelTests>(Obj);
	if (!FPaths::FileExists(ModelTests->ReimportPath))
	{
		return EReimportResult::Failed;
	}

	bool OutCanceled = false;
	if (ImportObject(ModelTests->GetClass(), ModelTests->GetOuter(), *ModelTests->GetName(), RF_Public | RF_Standalone, ModelTests->ReimportPath, nullptr, OutCanceled) != nullptr)
	{
		if (ModelTests->GetOuter())
		{
			ModelTests->GetOuter()->MarkPackageDirty();
		}
		else
		{
			ModelTests->MarkPackageDirty();
		}
		ModelTests->ReimportPath = "";
		return EReimportResult::Succeeded;
	}

	return EReimportResult::Failed;
}

UNNEModelTestsDataFactory::UNNEModelTestsDataFactory(const FObjectInitializer& ObjectInitializer) : UFactory(ObjectInitializer)
{
	bCreateNew = false;
	bEditorImport = true;
	SupportedClass = UNNEModelTestData::StaticClass();
	ImportPriority = DefaultImportPriority;
	Formats.Add("nned;NNE Model Test Data File");
}

UObject* UNNEModelTestsDataFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, TEXT("NNED"));

	TObjectPtr<UNNEModelTestData> ModelTestData = NewObject<UNNEModelTestData>(InParent, InClass, InName, Flags);
	if (!ModelTestData->InitializeFromFile(Filename))
	{
		return nullptr;
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, ModelTestData);

	return ModelTestData;
}

bool UNNEModelTestsDataFactory::FactoryCanImport(const FString& Filename)
{
	return Filename.EndsWith(FString(".nned"));
}