// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorReimportHandler.h"
#include "Factories/Factory.h"
#include "Logging/LogMacros.h"

#include "NNEModelTestsFactory.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogNNEModelTestsFactory, Log, All);

UCLASS()
class NNEMODELTESTSEDITOR_API UNNEModelTestsFactory : public UFactory, public FReimportHandler
{
	GENERATED_BODY()

public:
	UNNEModelTestsFactory(const FObjectInitializer& ObjectInitializer);

public:
	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	//~ End UFactory Interface

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	//~ End FReimportHandler Interface
};

UCLASS()
class NNEMODELTESTSEDITOR_API UNNEModelTestsDataFactory : public UFactory
{
	GENERATED_BODY()

public:
	UNNEModelTestsDataFactory(const FObjectInitializer& ObjectInitializer);

public:
	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	//~ End UFactory Interface
};